// Copyright Epic Games, Inc. All Rights Reserved.

#include "PLUGIN_NAME.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "HAL/PlatformProcess.h"
#include "PLUGIN_NAMELibrary/ExampleLibrary.h"

#define LOCTEXT_NAMESPACE "FPLUGIN_NAMEModule"

void FPLUGIN_NAMEModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	// Get the base directory of this plugin
	FString BaseDir = IPluginManager::Get().FindPlugin("PLUGIN_NAME")->GetBaseDir();

	// Add on the relative location of the third party dll and load it
	FString LibraryPath;
#if PLATFORM_WINDOWS
	LibraryPath = FPaths::Combine(*BaseDir, TEXT("Binaries/ThirdParty/PLUGIN_NAMELibrary/Win64/ExampleLibrary.dll"));
#elif PLATFORM_MAC
    LibraryPath = FPaths::Combine(*BaseDir, TEXT("Source/ThirdParty/PLUGIN_NAMELibrary/Mac/Release/libExampleLibrary.dylib"));
#elif PLATFORM_LINUX
	LibraryPath = FPaths::Combine(*BaseDir, TEXT("Binaries/ThirdParty/PLUGIN_NAMELibrary/Linux/x86_64-unknown-linux-gnu/libExampleLibrary.so"));
#endif // PLATFORM_WINDOWS

	ExampleLibraryHandle = !LibraryPath.IsEmpty() ? FPlatformProcess::GetDllHandle(*LibraryPath) : nullptr;

	if (ExampleLibraryHandle)
	{
		// Call the test function in the third party library that opens a message box
		ExampleLibraryFunction();
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ThirdPartyLibraryError", "Failed to load example third party library"));
	}
}

void FPLUGIN_NAMEModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	// Free the dll handle
	FPlatformProcess::FreeDllHandle(ExampleLibraryHandle);
	ExampleLibraryHandle = nullptr;
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FPLUGIN_NAMEModule, PLUGIN_NAME)
